// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPv4/IPv4Address.h"


/* FIpAddress4 static initialization
 *****************************************************************************/

const FIPv4Address FIPv4Address::Any(0, 0, 0, 0);
const FIPv4Address FIPv4Address::InternalLoopback(127, 0, 0, 1);
const FIPv4Address FIPv4Address::LanBroadcast(255, 255, 255, 255);


/* FIpAddress4 interface
 *****************************************************************************/

FString FIPv4Address::ToString() const
{
	return FString::Printf(TEXT("%i.%i.%i.%i"), A, B, C, D);
}


/* FIpAddress4 static interface
 *****************************************************************************/

bool FIPv4Address::Parse(const FString& AddressString, FIPv4Address& OutAddress)
{
	TArray<FString> Tokens;

	if (AddressString.ParseIntoArray(Tokens, TEXT("."), false) == 4)
	{
		const int32 A = FCString::Atoi(*Tokens[0]);
		const int32 B = FCString::Atoi(*Tokens[1]);
		const int32 C = FCString::Atoi(*Tokens[2]);
		const int32 D = FCString::Atoi(*Tokens[3]);

		if (A < 0 || A > MAX_uint8 || B < 0 || B > MAX_uint8 ||
			C < 0 || C > MAX_uint8 || D < 0 || D > MAX_uint8)
		{
			return false;
		}

		OutAddress.A = static_cast<uint8>(A);
		OutAddress.B = static_cast<uint8>(B);
		OutAddress.C = static_cast<uint8>(C);
		OutAddress.D = static_cast<uint8>(D);

		return true;
	}

	return false;
}
