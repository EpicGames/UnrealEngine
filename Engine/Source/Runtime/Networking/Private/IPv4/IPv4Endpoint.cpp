// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "AddressInfoTypes.h"


/* FIPv4Endpoint static initialization
 *****************************************************************************/

const FIPv4Endpoint FIPv4Endpoint::Any(FIPv4Address(0, 0, 0, 0), 0);
ISocketSubsystem* FIPv4Endpoint::CachedSocketSubsystem = nullptr;


/* FIPv4Endpoint interface
 *****************************************************************************/

FString FIPv4Endpoint::ToString() const
{
	return FString::Printf(TEXT("%s:%i"), *Address.ToString(), Port);
}


/* FIPv4Endpoint static interface
 *****************************************************************************/

void FIPv4Endpoint::Initialize()
{
	CachedSocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
}


bool FIPv4Endpoint::Parse(const FString& EndpointString, FIPv4Endpoint& OutEndpoint)
{
	TArray<FString> Tokens;

	if (EndpointString.ParseIntoArray(Tokens, TEXT(":"), false) == 2)
	{
		if (FIPv4Address::Parse(Tokens[0], OutEndpoint.Address))
		{
			const int32 Port = FCString::Atoi(*Tokens[1]);

			if (Port < 0 || Port > MAX_uint16)
			{
				return false;
			}

			OutEndpoint.Port = static_cast<uint16>(Port);

			return true;
		}
	}

	return false;
}

bool FIPv4Endpoint::FromHostAndPort(const FString& HostAndPortString, FIPv4Endpoint& OutEndpoint)
{
	FString Host;
	FString Port;
	if (!HostAndPortString.Split(TEXT(":"), &Host, &Port, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		return false;
	}

	Host.TrimStartAndEndInline();
	Port.TrimStartAndEndInline();
	if(Host.IsEmpty() || Port.IsEmpty())
	{
		return false;
	}

	const FAddressInfoResult AddressInfo = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressInfo(
		*Host,
		*Port,
		EAddressInfoFlags::NoResolveService,
		FNetworkProtocolTypes::IPv4,
		SOCKTYPE_Unknown);
	if (AddressInfo.ReturnCode != SE_NO_ERROR || AddressInfo.Results.Num() == 0)
	{
		return false;
	}

	const FAddressInfoResultData& Result = AddressInfo.Results[0];
	OutEndpoint = FIPv4Endpoint(Result.Address);
	return true;
}
