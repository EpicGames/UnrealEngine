// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPv4/IPv4Subnet.h"


/* FIPv4Subnet interface
 *****************************************************************************/

FString FIPv4Subnet::ToString() const
{
	return FString::Printf(TEXT("%s/%s"), *Address.ToString(), *Mask.ToString());
}


/* FIPv4Subnet static interface
 *****************************************************************************/

bool FIPv4Subnet::Parse(const FString& SubnetString, FIPv4Subnet& OutSubnet)
{
	TArray<FString> Tokens;

	if (SubnetString.ParseIntoArray(Tokens, TEXT("/"), false) == 2)
	{
		return (FIPv4Address::Parse(Tokens[0], OutSubnet.Address) && FIPv4SubnetMask::Parse(Tokens[1], OutSubnet.Mask));
	}

	return false;
}
