// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPv4/IPv4SubnetMask.h"


/* FIpAddress4 interface
 *****************************************************************************/

FString FIPv4SubnetMask::ToString() const
{
	return FString::Printf(TEXT("%i.%i.%i.%i"), A, B, C, D);
}


/* FIpAddress4 static interface
 *****************************************************************************/

bool FIPv4SubnetMask::Parse(const FString& MaskString, FIPv4SubnetMask& OutMask)
{
	TArray<FString> Tokens;

	if (MaskString.ParseIntoArray(Tokens, TEXT("."), false) == 4)
	{
		const int32 A = FCString::Atoi(*Tokens[0]);
		const int32 B = FCString::Atoi(*Tokens[1]);
		const int32 C = FCString::Atoi(*Tokens[2]);
		const int32 D = FCString::Atoi(*Tokens[3]);

		if (A < 0 || A > MAX_uint8 || B < 0 || B > MAX_uint8 ||
			C < 0 || C > MAX_uint8 || D < 0 || D > MAX_uint8)
		{
			return false;
		}

		OutMask.A = static_cast<uint8>(A);
		OutMask.B = static_cast<uint8>(B);
		OutMask.C = static_cast<uint8>(C);
		OutMask.D = static_cast<uint8>(D);

		return true;
	}

	return false;
}
