// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkingModule.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"


IMPLEMENT_MODULE(FNetworkingModule, Networking);


/* IModuleInterface interface
 *****************************************************************************/

void FNetworkingModule::StartupModule()
{
	FIPv4Endpoint::Initialize();
}


void FNetworkingModule::ShutdownModule()
{
}


bool FNetworkingModule::SupportsDynamicReloading()
{
	return false;
}
