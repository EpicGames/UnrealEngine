// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/INetworkingModule.h"

/**
 * Implements the Networking module.
 */
class FNetworkingModule
	: public INetworkingModule
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
};
