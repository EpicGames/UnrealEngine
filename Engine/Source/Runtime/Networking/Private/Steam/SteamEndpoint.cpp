// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/Steam/SteamEndpoint.h"


/* FSteamEndpoint interface
 *****************************************************************************/

FString FSteamEndpoint::ToString() const
{
	return FString::Printf(TEXT("0x%llX:%i"), UniqueNetId, SteamChannel);
}
