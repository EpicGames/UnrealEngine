// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Interfaces/IPv4/IPv4Address.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIPv4AddressTest, "System.Engine.Networking.IPv4.IPv4Address", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)


bool FIPv4AddressTest::RunTest( const FString& Parameters )
{
	// component access must be correct
	FIPv4Address a1_1 = FIPv4Address(1, 2, 3, 4);

	TestEqual<uint8>(TEXT("Component A of 1.2.3.4 must be 1"), a1_1.A, 1);
	TestEqual<uint8>(TEXT("Component B of 1.2.3.4 must be 2"), a1_1.B, 2);
	TestEqual<uint8>(TEXT("Component C of 1.2.3.4 must be 3"), a1_1.C, 3);
	TestEqual<uint8>(TEXT("Component D of 1.2.3.4 must be 4"), a1_1.D, 4);

	// link local addresses must be recognized
	FIPv4Address a2_1 = FIPv4Address(169, 254, 0, 1);
	FIPv4Address a2_2 = FIPv4Address(168, 254, 0, 1);
	FIPv4Address a2_3 = FIPv4Address(169, 253, 0, 1);

	TestTrue(TEXT("169.254.0.1 must be a link local address"), a2_1.IsLinkLocal());
	TestFalse(TEXT("168.254.0.1 must not be a link local address"), a2_2.IsLinkLocal());
	TestFalse(TEXT("169.253.0.1 must not be a link local address"), a2_3.IsLinkLocal());

	// loopback addresses must be recognized
	FIPv4Address a3_1 = FIPv4Address(127, 0, 0, 1);
	FIPv4Address a3_2 = FIPv4Address(128, 0, 0, 1);

	TestTrue(TEXT("127.0.0.1 must be a loopback address"), a3_1.IsLoopbackAddress());
	TestFalse(TEXT("128.0.0.1 must not be a loopback address"), a3_2.IsLoopbackAddress());

	// multicast addresses must be recognized
	FIPv4Address a4_1 = FIPv4Address(223, 255, 255, 255);
	FIPv4Address a4_2 = FIPv4Address(224, 0, 0, 0);
	FIPv4Address a4_3 = FIPv4Address(239, 255, 255, 255);
	FIPv4Address a4_4 = FIPv4Address(240, 0, 0, 0);

	TestFalse(TEXT("223.255.255.255 must not be a multicast address"), a4_1.IsMulticastAddress());
	TestTrue(TEXT("224.0.0.0 must be a multicast address"), a4_2.IsMulticastAddress());
	TestTrue(TEXT("239.255.255.255 must be a multicast address"), a4_3.IsMulticastAddress());
	TestFalse(TEXT("240.0.0.0 must not be a multicast address"), a4_4.IsMulticastAddress());

	// string conversion
	FIPv4Address a5_1 = FIPv4Address(1, 2, 3, 4);

	TestEqual<FString>(TEXT("String conversion (1.2.3.4)"), a5_1.ToString(), TEXT("1.2.3.4"));

	// parsing valid strings must succeed
	FIPv4Address a6_1 = FIPv4Address(1, 2, 3, 4);
	FIPv4Address a6_2;

	TestTrue(TEXT("Parsing valid strings must succeed (1.2.3.4)"), FIPv4Address::Parse(TEXT("1.2.3.4"), a6_2));
	TestEqual(TEXT("Parsing valid strings must result in correct value (1.2.3.4)"), a6_2, a6_1);

	// parsing invalid strings must fail
	FIPv4Address a7_1;

	TestFalse(TEXT("Parsing invalid strings must fail (empty)"), FIPv4Address::Parse(TEXT(""), a7_1));
//	TestFalse(TEXT("Parsing invalid strings must fail (...)"), FIPv4Address::Parse(TEXT("..."), a7_1));
	TestFalse(TEXT("Parsing invalid strings must fail (1.2.3)"), FIPv4Address::Parse(TEXT("1.2.3"), a7_1));
//	TestFalse(TEXT("Parsing invalid strings must fail (1.2.3.)"), FIPv4Address::Parse(TEXT("1.2.3."), a7_1));
	TestFalse(TEXT("Parsing invalid strings must fail (1.2.3.4.)"), FIPv4Address::Parse(TEXT("1.2.3.4."), a7_1));
	TestFalse(TEXT("Parsing invalid strings must fail (.1.2.3.4)"), FIPv4Address::Parse(TEXT(".1.2.3.4"), a7_1));

	// site local addresses must be recognized
	FIPv4Address a8_1 = FIPv4Address(10, 0, 0, 1);
	FIPv4Address a8_2 = FIPv4Address(172, 16, 0, 1);
	FIPv4Address a8_3 = FIPv4Address(172, 31, 0, 1);
	FIPv4Address a8_4 = FIPv4Address(192, 168, 0, 1);

	TestTrue(TEXT("10.0.0.1 must be a site local address"), a8_1.IsSiteLocalAddress());
	TestTrue(TEXT("172.16.0.1 must be a site local address"), a8_2.IsSiteLocalAddress());
	TestTrue(TEXT("172.31.0.1 must be a site local address"), a8_3.IsSiteLocalAddress());
	TestTrue(TEXT("192.168.0.1 must be a site local address"), a8_4.IsSiteLocalAddress());

	FIPv4Address a8_5 = FIPv4Address(11, 0, 0, 1);
	FIPv4Address a8_6 = FIPv4Address(173, 16, 0, 1);
	FIPv4Address a8_7 = FIPv4Address(172, 32, 0, 1);
	FIPv4Address a8_8 = FIPv4Address(193, 168, 0, 1);
	FIPv4Address a8_9 = FIPv4Address(192, 169, 0, 1);

	TestFalse(TEXT("11.0.0.1 must not be a site local address"), a8_5.IsSiteLocalAddress());
	TestFalse(TEXT("173.16.0.1 must not be a site local address"), a8_6.IsSiteLocalAddress());
	TestFalse(TEXT("172.32.0.1 must not be a site local address"), a8_7.IsSiteLocalAddress());
	TestFalse(TEXT("193.168.0.1 must not be a site local address"), a8_8.IsSiteLocalAddress());
	TestFalse(TEXT("192.169.0.1 must not be a site local address"), a8_9.IsSiteLocalAddress());

	// host byte order integer constructor
#if PLATFORM_LITTLE_ENDIAN
	FIPv4Address a9_1(3232235521);	// 192.168.0.1 in little-endian
#else
	FIPv4Address a9_1(16820416);	// 192.168.0.1 in big-endian
#endif
	FIPv4Address a9_2(192, 168, 0, 1);

	TestEqual<FIPv4Address>(TEXT("Host byte order constructor must yield correct components"), a9_1, a9_2);

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
