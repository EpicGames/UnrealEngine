// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "IPAddress.h"
#include "SocketSubsystem.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "HAL/RunnableThread.h"
#include "Common/TcpSocketBuilder.h"

/**
 * Delegate type for accepted TCP connections.
 *
 * The first parameter is the socket for the accepted connection.
 * The second parameter is the remote IP endpoint of the accepted connection.
 * The return value indicates whether the connection should be accepted.
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnTcpListenerConnectionAccepted, FSocket*, const FIPv4Endpoint&)


/**
 * Implements a runnable that listens for incoming TCP connections.
 */
class FTcpListener
	: public FRunnable
{
public:

	/**
	 * Creates and initializes a new instance from the specified IP endpoint.
	 *
	 * @param LocalEndpoint The local IP endpoint to listen on.
	 * @param SleepTime The maximum time to wait for a pending connection inside the polling loop (default = 1 second).
	 * @param Toggles socket-reusability which allows for other sockets to use the same address (default = true).
	 */
	FTcpListener(const FIPv4Endpoint& LocalEndpoint, const FTimespan& InSleepTime = FTimespan::FromSeconds(1), bool bInReusable = true)
		: bDeleteSocket(true)
		, Endpoint(LocalEndpoint)
		, SleepTime(InSleepTime)
		, Socket(nullptr)
		, bStopping(false)
		, bSocketReusable(bInReusable)
	{
		Thread = FRunnableThread::Create(this, TEXT("FTcpListener"), 8 * 1024, TPri_Normal);
	}

	/**
	 * Creates and initializes a new instance from the specified socket.
	 *
	 * @param InSocket The socket to listen on.
	 * @param SleepTime SleepTime The maximum time to wait for a pending connection inside the polling loop (default = 1 second).
	 * @param bInReusable Toggles socket-reusability which allows for other sockets to use the same address (default = true).
	 */
	FTcpListener(FSocket& InSocket, const FTimespan& InSleepTime = FTimespan::FromSeconds(1), bool bInReusable = true)
		: bDeleteSocket(false)
		, SleepTime(InSleepTime)
		, Socket(&InSocket)
		, bStopping(false)
		, bSocketReusable(bInReusable)
	{
		TSharedRef<FInternetAddr> LocalAddress = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
		Socket->GetAddress(*LocalAddress);
		Endpoint = FIPv4Endpoint(LocalAddress);

		Thread = FRunnableThread::Create(this, TEXT("FTcpListener"), 8 * 1024, TPri_Normal);
	}

	/** Destructor. */
	~FTcpListener()
	{
		if (Thread != nullptr)
		{
			Thread->Kill(true);
			delete Thread;
		}

		if (bDeleteSocket && (Socket != nullptr))
		{
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
			Socket = nullptr;
		}
	}

public:

	/**
	 * Gets the listener's local IP endpoint.
	 *
	 * @return IP endpoint.
	 */
	const FIPv4Endpoint& GetLocalEndpoint() const
	{
		return Endpoint;
	}

	/**
	 * Gets the listener's network socket.
	 *
	 * @return Network socket.
	 */
	FSocket* GetSocket() const
	{
		return Socket;
	}

	/**
	 * Checks whether the listener is listening for incoming connections.
	 *
	 * @return true if it is listening, false otherwise.
	 */
	bool IsActive() const
	{
		return ((Socket != nullptr) && !bStopping);
	}

public:

	/**
	 * Gets a delegate to be invoked when an incoming connection has been accepted.
	 *
	 * If this delegate is not bound, the listener will reject all incoming connections.
	 * To temporarily disable accepting connections, use the Enable() and Disable() methods.
	 *
	 * @return The delegate.
	 */
	FOnTcpListenerConnectionAccepted& OnConnectionAccepted()
	{
		return ConnectionAcceptedDelegate;
	}

public:

	// FRunnable interface

	virtual bool Init() override
	{
		if (Socket == nullptr)
		{
			Socket = FTcpSocketBuilder(TEXT("FTcpListener server"))
				.AsReusable(bSocketReusable)
				.BoundToEndpoint(Endpoint)
				.Listening(8)
				.WithSendBufferSize(2 * 1024 * 1024);
		}

		return (Socket != nullptr);
	}

	virtual uint32 Run() override
	{
		TSharedRef<FInternetAddr> RemoteAddress = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();

		const bool bHasZeroSleepTime = (SleepTime == FTimespan::Zero());

		while (!bStopping)
		{
			bool Pending = false;

			// handle incoming connections
			if (Socket->WaitForPendingConnection(Pending, SleepTime))
			{
				if (Pending)
				{
					FSocket* ConnectionSocket = Socket->Accept(*RemoteAddress, TEXT("FTcpListener client"));

					if (ConnectionSocket != nullptr)
					{
						bool Accepted = false;

						if (ConnectionAcceptedDelegate.IsBound())
						{
							Accepted = ConnectionAcceptedDelegate.Execute(ConnectionSocket, FIPv4Endpoint(RemoteAddress));
						}

						if (!Accepted)
						{
							ConnectionSocket->Close();
							ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(ConnectionSocket);
						}
					}
				}
				else if(bHasZeroSleepTime)
				{
					FPlatformProcess::Sleep(0.f);
				}
			}
			else
			{
				FPlatformProcess::Sleep(static_cast<float>(SleepTime.GetSeconds()));
			}
		}

		return 0;
	}

	virtual void Stop() override
	{
		bStopping = true;
	}

	virtual void Exit() override { }

private:

	/** Holds a flag indicating whether the socket should be deleted in the destructor. */
	bool bDeleteSocket;

	/** Holds the server endpoint. */
	FIPv4Endpoint Endpoint;

	/** Holds the time to sleep between checking for pending connections. */
	FTimespan SleepTime;

	/** Holds the server socket. */
	FSocket* Socket;

	/** Holds a flag indicating that the thread is stopping. */
	bool bStopping;

	/** Holds a flag indicating if the bound address can be used by other sockets. */
	bool bSocketReusable;

	/** Holds the thread object. */
	FRunnableThread* Thread;

private:

	/** Holds a delegate to be invoked when an incoming connection has been accepted. */
	FOnTcpListenerConnectionAccepted ConnectionAcceptedDelegate;
};
