// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OutputDeviceRedirector.h"
#include "SocketSubsystem.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Sockets.h"
#include "SocketTypes.h"

class Error;

/**
 * Implements a fluent builder for TCP sockets.
 */
class FTcpSocketBuilder
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InDescription Debug description for the socket.
	 */
	FTcpSocketBuilder(const FString& InDescription)
		: Blocking(false)
		, Bound(false)
		, BoundEndpoint(FIPv4Address::Any, 0)
		, Description(InDescription)
		, Linger(false)
		, LingerTimeout(0)
		, Listen(false)
		, ReceiveBufferSize(0)
		, Reusable(false)
		, SendBufferSize(0)
	{ }

public:

	/**
	 * Sets socket operations to be blocking.
	 *
	 * @return This instance (for method chaining).
	 * @see AsNonBlocking, AsReusable
	 */
	FTcpSocketBuilder AsBlocking()
	{
		Blocking = true;

		return *this;
	}

	/**
	 * Sets socket operations to be non-blocking.
	 *
	 * @return This instance (for method chaining).
	 * @see AsBlocking, AsReusable
	 */
	FTcpSocketBuilder AsNonBlocking()
	{
		Blocking = false;

		return *this;
	}

	/**
	 * Makes the bound address reusable by other sockets.
	 *
	 * @return This instance (for method chaining).
	 * @see AsNonBlocking, AsNonBlocking
	 */
	FTcpSocketBuilder AsReusable()
	{
		Reusable = true;

		return *this;
	}

	/**
	* Sets re-usability of the bound address by other sockets.
	*
	* @param bInReusable Whether to enable or disable re-usability of the bound address.
	* @return This instance (for method chaining).
	* @see AsNonBlocking, AsNonBlocking
	*/
	FTcpSocketBuilder AsReusable(bool bInReusable)
	{
		Reusable = bInReusable;

		return *this;
	}

	/**
 	 * Sets the local address to bind the socket to.
	 *
	 * Unless specified in a subsequent call to BoundToPort(), a random
	 * port number will be assigned by the underlying provider.
	 *
	 * @param Address The IP address to bind the socket to.
	 * @return This instance (for method chaining).
	 * @see BoundToEndpoint, BoundToPort
	 */
	FTcpSocketBuilder BoundToAddress(const FIPv4Address& Address)
	{
		BoundEndpoint = FIPv4Endpoint(Address, BoundEndpoint.Port);
		Bound = true;

		return *this;
	}

	/**
 	 * Sets the local endpoint to bind the socket to.
	 *
	 * @param Endpoint The IP endpoint to bind the socket to.
	 * @return This instance (for method chaining).
	 * @see BoundToAddress, BoundToPort
	 */
	FTcpSocketBuilder BoundToEndpoint(const FIPv4Endpoint& Endpoint)
	{
		BoundEndpoint = Endpoint;
		Bound = true;

		return *this;
	}

	/**
	 * Sets the local port to bind the socket to.
	 *
	 * Unless specified in a subsequent call to BoundToAddress(), the local
	 * address will be determined automatically by the underlying provider.
	 *
	 * @param Port The local port number to bind the socket to.
	 * @return This instance (for method chaining).
	 * @see BoundToAddress, BoundToEndpoint
	 */
	FTcpSocketBuilder BoundToPort(uint16 Port)
	{
		BoundEndpoint = FIPv4Endpoint(BoundEndpoint.Address, Port);
		Bound = true;

		return *this;
	}

	/**
	 * Sets how long the socket will linger after closing.
	 *
	 * @param Timeout The amount of time to linger before closing.
	 * @return This instance (for method chaining).
	 */
	FTcpSocketBuilder Lingering(int32 Timeout)
	{
		Linger = true;
		LingerTimeout = Timeout;

		return *this;
	}

	/**
	 * Sets the socket into a listening state for incoming connections.
	 *
	 * @param MaxBacklog The number of connections to queue before refusing them.
	 * @return This instance (for method chaining).
	 */
	FTcpSocketBuilder Listening(int32 MaxBacklog)
	{
		Listen = true;
		ListenBacklog = MaxBacklog;

		return *this;
	}

	/**
	 * Specifies the desired size of the receive buffer in bytes (0 = default).
	 *
	 * The socket creation will not fail if the desired size cannot be set or
	 * if the actual size is less than the desired size.
	 *
	 * @param SizeInBytes The size of the buffer.
	 * @return This instance (for method chaining).
	 * @see WithSendBufferSize
	 */
	FTcpSocketBuilder WithReceiveBufferSize(int32 SizeInBytes)
	{
		ReceiveBufferSize = SizeInBytes;

		return *this;
	}

	/**
	 * Specifies the desired size of the send buffer in bytes (0 = default).
	 *
	 * The socket creation will not fail if the desired size cannot be set or
	 * if the actual size is less than the desired size.
	 *
	 * @param SizeInBytes The size of the buffer.
	 * @return This instance (for method chaining).
	 * @see WithReceiveBufferSize
	 */
	FTcpSocketBuilder WithSendBufferSize(int32 SizeInBytes)
	{
		SendBufferSize = SizeInBytes;

		return *this;
	}

public:

	/**
	 * Implicit conversion operator that builds the socket as configured.
	 *
	 * @return The built socket.
	 */
	operator FSocket*() const
	{
		return Build();
	}

	/**
	 * Builds the socket as configured.
	 *
	 * @return The built socket.
	 */
	FSocket* Build() const
	{
		FSocket* Socket = nullptr;

		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

		if (SocketSubsystem != nullptr)
		{
			TSharedRef<FInternetAddr> BoundEndpointAddr = BoundEndpoint.ToInternetAddr();
			Socket = SocketSubsystem->CreateSocket(NAME_Stream, *Description, BoundEndpointAddr->GetProtocolType());

			if (Socket != nullptr)
			{
				bool Error = !Socket->SetReuseAddr(Reusable) ||
							 !Socket->SetLinger(Linger, LingerTimeout) ||
							 !Socket->SetRecvErr();

				if (!Error)
				{
					Error = Bound && !Socket->Bind(*BoundEndpointAddr);
				}

				if (!Error)
				{
					Error = Listen && !Socket->Listen(ListenBacklog);
				}

				if (!Error)
				{
					Error = !Socket->SetNonBlocking(!Blocking);
				}

				if (!Error)
				{
					int32 OutNewSize;

					if (ReceiveBufferSize > 0)
					{
						Socket->SetReceiveBufferSize(ReceiveBufferSize, OutNewSize);
					}

					if (SendBufferSize > 0)
					{
						Socket->SetSendBufferSize(SendBufferSize, OutNewSize);
					}
				}

				if (Error)
				{
					GLog->Logf(TEXT("FTcpSocketBuilder: Failed to create the socket %s as configured"), *Description);

					SocketSubsystem->DestroySocket(Socket);

					Socket = nullptr;
				}
			}
		}

		return Socket;
	}

private:

	/** Holds a flag indicating whether socket operations are blocking. */
	bool Blocking;

	/** Holds a flag indicating whether the socket should be bound. */
	bool Bound;

	/** Holds the IP address (and port) that the socket will be bound to. */
	FIPv4Endpoint BoundEndpoint;

	/** Holds the socket's debug description text. */
	FString Description;

	/** Holds a flag indicating whether the socket should linger after closing. */
	bool Linger;

	/** Holds the amount of time the socket will linger before closing. */
	int32 LingerTimeout;

	/** Holds a flag indicating whether the socket should listen for incoming connections. */
	bool Listen;

	/** Holds the number of connections to queue up before refusing them. */
	int32 ListenBacklog;

	/** The desired size of the receive buffer in bytes (0 = default). */
	int32 ReceiveBufferSize;

	/** Holds a flag indicating whether the bound address can be reused by other sockets. */
	bool Reusable;

	/** The desired size of the send buffer in bytes (0 = default). */
	int32 SendBufferSize;
};
