// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

class Error;

/**
 * Implements a fluent builder for UDP sockets.
 */
class FUdpSocketBuilder
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InDescription Debug description for the socket.
	 */
	FUdpSocketBuilder(const FString& InDescription)
		: AllowBroadcast(false)
		, Blocking(false)
		, Bound(false)
		, BoundEndpoint(FIPv4Address::Any, 0)
		, Description(InDescription)
		, MulticastInterface(FIPv4Address::Any)
		, MulticastLoopback(false)
		, MulticastTtl(1)
		, ReceiveBufferSize(0)
		, Reusable(false)
		, SendBufferSize(0)
	{ }

public:

	/**
	 * Sets socket operations to be blocking.
	 *
	 * @return This instance (for method chaining).
	 * @see AsNonBlocking, AsReusable
	 */
	FUdpSocketBuilder& AsBlocking()
	{
		Blocking = true;

		return *this;
	}

	/**
	 * Sets socket operations to be non-blocking.
	 *
	 * @return This instance (for method chaining).
	 * @see AsBlocking, AsReusable
	 */
	FUdpSocketBuilder& AsNonBlocking()
	{
		Blocking = false;

		return *this;
	}

	/**
	 * Makes the bound address reusable by other sockets.
	 *
	 * @return This instance (for method chaining).
	 * @see AsBlocking, AsNonBlocking
	 */
	FUdpSocketBuilder& AsReusable()
	{
		Reusable = true;

		return *this;
	}

	/**
 	 * Sets the local address to bind the socket to.
	 *
	 * Unless specified in a subsequent call to BoundToPort(), a random
	 * port number will be assigned by the underlying provider.
	 *
	 * @param Address The IP address to bind the socket to.
	 * @return This instance (for method chaining).
	 * @see BoundToEndpoint, BoundToPort
	 */
	FUdpSocketBuilder& BoundToAddress(const FIPv4Address& Address)
	{
		BoundEndpoint = FIPv4Endpoint(Address, BoundEndpoint.Port);
		Bound = true;

		return *this;
	}

	/**
 	 * Sets the local endpoint to bind the socket to.
	 *
	 * @param Endpoint The IP endpoint to bind the socket to.
	 * @return This instance (for method chaining).
	 * @see BoundToAddress, BoundToPort
	 */
	FUdpSocketBuilder& BoundToEndpoint(const FIPv4Endpoint& Endpoint)
	{
		BoundEndpoint = Endpoint;
		Bound = true;

		return *this;
	}

	/**
	 * Sets the local port to bind the socket to.
	 *
	 * Unless specified in a subsequent call to BoundToAddress(), the local
	 * address will be determined automatically by the underlying provider.
	 *
	 * @param Port The local port number to bind the socket to.
	 * @return This instance (for method chaining).
	 * @see BoundToAddress
	 */
	FUdpSocketBuilder& BoundToPort(uint16 Port)
	{
		BoundEndpoint = FIPv4Endpoint(BoundEndpoint.Address, Port);
		Bound = true;

		return *this;
	}

	/**
	 * Joins the socket to the specified multicast group.
	 *
	 * @param GroupAddress The IP address of the multicast group to join.
	 * @return This instance (for method chaining).
	 * @see WithMulticastLoopback, WithMulticastTtl
	 */
	FUdpSocketBuilder& JoinedToGroup(const FIPv4Address& GroupAddress)
	{
		return JoinedToGroup(GroupAddress, FIPv4Address::Any);
	}

	/**
	 * Joins the socket to the specified multicast group.
	 *
	 * @param GroupAddress The IP address of the multicast group to join.
	 * @param InterfaceAddress The IP address of the interface to join the multicast group on.
	 * @return This instance (for method chaining).
	 * @see WithMulticastLoopback, WithMulticastTtl
	 */
	FUdpSocketBuilder& JoinedToGroup(const FIPv4Address& GroupAddress, const FIPv4Address& InterfaceAddress)
	{
		JoinedGroups.Add({ GroupAddress, InterfaceAddress });

		return *this;
	}

	/**
	 * Enables broadcasting.
	 *
	 * @return This instance (for method chaining).
	 */
	FUdpSocketBuilder& WithBroadcast()
	{
		AllowBroadcast = true;

		return *this;
	}

	/**
	 * Enables multicast loopback.
	 *
	 * @param AllowLoopback Whether to allow multicast loopback.
	 * @param TimeToLive The time to live.
	 * @return This instance (for method chaining).
	 * @see JoinedToGroup, WithMulticastTtl
	 */
	FUdpSocketBuilder& WithMulticastLoopback()
	{
		MulticastLoopback = true;

		return *this;
	}

	/**
	 * Sets the multicast time-to-live.
	 *
	 * @param TimeToLive The time to live.
	 * @return This instance (for method chaining).
	 * @see JoinedToGroup, WithMulticastLoopback
	 */
	FUdpSocketBuilder& WithMulticastTtl(uint8 TimeToLive)
	{
		MulticastTtl = TimeToLive;

		return *this;
	}

	/**
	 * Sets the multicast outgoing interface.
	 *
	 * @param InterfaceAddress The interface to use to send multicast datagrams.
	 * @return This instance (for method chaining).
	 * @see JoinedToGroup, WithMulticastLoopback
	 */
	FUdpSocketBuilder& WithMulticastInterface(const FIPv4Address& InterfaceAddress)
	{
		MulticastInterface = InterfaceAddress;

		return *this;
	}


	/**
	 * Specifies the desired size of the receive buffer in bytes (0 = default).
	 *
	 * The socket creation will not fail if the desired size cannot be set or
	 * if the actual size is less than the desired size.
	 *
	 * @param SizeInBytes The size of the buffer.
	 * @return This instance (for method chaining).
	 * @see WithSendBufferSize
	 */
	FUdpSocketBuilder& WithReceiveBufferSize(int32 SizeInBytes)
	{
		ReceiveBufferSize = SizeInBytes;

		return *this;
	}

	/**
	 * Specifies the desired size of the send buffer in bytes (0 = default).
	 *
	 * The socket creation will not fail if the desired size cannot be set or
	 * if the actual size is less than the desired size.
	 *
	 * @param SizeInBytes The size of the buffer.
	 * @return This instance (for method chaining).
	 * @see WithReceiveBufferSize
	 */
	FUdpSocketBuilder& WithSendBufferSize(int32 SizeInBytes)
	{
		SendBufferSize = SizeInBytes;

		return *this;
	}

public:

	/**
	 * Implicit conversion operator that builds the socket as configured.
	 *
	 * @return The built socket.
	 */
	operator FSocket*() const
	{
		return Build();
	}

	/**
	 * Builds the socket as configured.
	 *
	 * @return The built socket.
	 */
	FSocket* Build() const
	{
		// load socket subsystem
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

		if (SocketSubsystem == nullptr)
		{
			GLog->Log(TEXT("FUdpSocketBuilder: Failed to load socket subsystem"));
			return nullptr;
		}

		// create socket
		TSharedRef<FInternetAddr> RemoteAddr = BoundEndpoint.ToInternetAddr();
		FSocket* Socket = SocketSubsystem->CreateSocket(NAME_DGram, *Description, RemoteAddr->GetProtocolType());

		if (Socket == nullptr)
		{
			GLog->Logf(TEXT("FUdpSocketBuilder: Failed to create socket %s"), *Description);
			return nullptr;
		}

		// configure socket

		if (!Socket->SetNonBlocking(!Blocking) ||
			!Socket->SetReuseAddr(Reusable) ||
			!Socket->SetBroadcast(AllowBroadcast) ||
			!Socket->SetRecvErr())
		{
			const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

			GLog->Logf(TEXT("FUdpSocketBuilder: Failed to configure %s (blocking: %i, reusable: %i, broadcast: %i). Error code %d."), 
				*Description, Blocking, Reusable, AllowBroadcast, int32(SocketError));

			SocketSubsystem->DestroySocket(Socket);
			return nullptr;
		}

		// bind socket

		if (Bound && !Socket->Bind(*RemoteAddr))
		{
			const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

			GLog->Logf(TEXT("FUdpSocketBuilder: Failed to bind %s to %s. Error code %d."),
				*Description, *BoundEndpoint.ToString(), int32(SocketError));

			SocketSubsystem->DestroySocket(Socket);
			return nullptr;
		}

		// configure multicast

		if (!Socket->SetMulticastLoopback(MulticastLoopback) || !Socket->SetMulticastTtl(MulticastTtl))
		{
			const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

			GLog->Logf(TEXT("FUdpSocketBuilder: Failed to configure multicast for %s (loopback: %i, ttl: %i). Error code %d."),
				*Description, MulticastLoopback, MulticastTtl, int32(SocketError));

			SocketSubsystem->DestroySocket(Socket);
			return nullptr;
		}

		// join multicast groups

		TSharedRef<FInternetAddr> MulticastAddress = SocketSubsystem->CreateInternetAddr(RemoteAddr->GetProtocolType());
		MulticastAddress->SetBroadcastAddress();
		TSharedPtr<FInternetAddr> AddressToUse = nullptr;

		for (const auto& Group : JoinedGroups)
		{
			// The Socket code no longer has the multicast address hack handled anymore, as such, we need to properly determine the address ourselves.
			if (Group.GroupAddress.IsSessionFrontendMulticast() && MulticastAddress->GetProtocolType() != FNetworkProtocolTypes::IPv4)
			{
				// This will use the address protocol that we figured out earlier.
				AddressToUse = MulticastAddress;
			}
			else
			{
				// Otherwise, we'll use the address we use all the time.
				AddressToUse = FIPv4Endpoint(Group.GroupAddress, 0).ToInternetAddr();
			}

			if (!Socket->JoinMulticastGroup(*AddressToUse, *FIPv4Endpoint(Group.InterfaceAddress, 0).ToInternetAddr()))
			{
				const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

				GLog->Logf(TEXT("FUdpSocketBuilder: Failed to subscribe %s to multicast group %s on interface %s. Error code %d."),
					*Description, *Group.GroupAddress.ToString(), *Group.InterfaceAddress.ToString(), int32(SocketError));

				SocketSubsystem->DestroySocket(Socket);
				return nullptr;
			}
		}

		// set multicast outgoing interface
		if (MulticastInterface != FIPv4Address::Any)
		{
			if (!Socket->SetMulticastInterface(*FIPv4Endpoint(MulticastInterface, 0).ToInternetAddr()))
			{
				const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

				GLog->Logf(TEXT("FUdpSocketBuilder: Failed to set multicast outgoing interface for %s to %s. Error code %d."),
					*Description, *MulticastInterface.ToString(), int32(SocketError));

				SocketSubsystem->DestroySocket(Socket);
				return nullptr;
			}
		}

		// set buffer sizes
		{
			int32 OutNewSize;

			if (ReceiveBufferSize > 0)
			{
				if (!Socket->SetReceiveBufferSize(ReceiveBufferSize, OutNewSize))
				{
					const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

					GLog->Logf(TEXT("FUdpSocketBuilder: Warning - could not set receive buffer size to %d for %s. Error code %d."),
						ReceiveBufferSize, *Description, int32(SocketError));
				}
			}

			if (SendBufferSize > 0)
			{
				if (!Socket->SetSendBufferSize(SendBufferSize, OutNewSize))
				{
					const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();

					GLog->Logf(TEXT("FUdpSocketBuilder: Warning - could not set send buffer size to %d for %s. Error code %d."),
						SendBufferSize, *Description, int32(SocketError));
				}
			}
		}

		return Socket;
	}

private:

	/** Holds a flag indicating whether broadcasts will be enabled. */
	bool AllowBroadcast;

	/** Holds a flag indicating whether socket operations are blocking. */
	bool Blocking;

	/** Holds a flag indicating whether the socket should be bound. */
	bool Bound;

	/** Holds the IP address (and port) that the socket will be bound to. */
	FIPv4Endpoint BoundEndpoint;

	/** Holds the socket's debug description text. */
	FString Description;

	/** Holds the IP address of the interface to use for outgoing multicast datagrams. */
	FIPv4Address MulticastInterface;

	/** Holds the list of joined multicast groups. */
	struct FMulticastGroup
	{
		FIPv4Address GroupAddress;
		FIPv4Address InterfaceAddress;
	};
	TArray<FMulticastGroup> JoinedGroups;

	/** Holds a flag indicating whether multicast loopback will be enabled. */
	bool MulticastLoopback;

	/** Holds the multicast time to live. */
	uint8 MulticastTtl;

	/** The desired size of the receive buffer in bytes (0 = default). */
	int32 ReceiveBufferSize;

	/** Holds a flag indicating whether the bound address can be reused by other sockets. */
	bool Reusable;

	/** The desired size of the send buffer in bytes (0 = default). */
	int32 SendBufferSize;
};
