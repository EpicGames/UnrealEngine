// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "Serialization/ArrayReader.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IPAddress.h"

#include <atomic>

/**
 * Temporary fix for concurrency crashes. This whole class will be redesigned.
 */
typedef TSharedPtr<FArrayReader, ESPMode::ThreadSafe> FArrayReaderPtr;

/**
 * Delegate type for received data.
 *
 * The first parameter is the received data.
 * The second parameter is sender's IP endpoint.
 */
DECLARE_DELEGATE_TwoParams(FOnSocketDataReceived, const FArrayReaderPtr&, const FIPv4Endpoint&);


/**
 * Asynchronously receives data from an UDP socket.
 */
class FUdpSocketReceiver
	: public FRunnable
	, private FSingleThreadRunnable
{
public:

	/**
	 * Creates and initializes a new socket receiver.
	 *
	 * @param InSocket The UDP socket to receive data from.
	 * @param InWaitTime The amount of time to wait for the socket to be readable.
	 * @param InThreadName The receiver thread name (for debugging).
	 */
	FUdpSocketReceiver(FSocket* InSocket, const FTimespan& InWaitTime, const TCHAR* InThreadName)
		: Socket(InSocket)
		, bStopping(false)
		, Thread(nullptr)
		, ThreadName(InThreadName)
		, WaitTime(InWaitTime)
	{
		check(Socket != nullptr);
		check(Socket->GetSocketType() == SOCKTYPE_Datagram);

		SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	}

	/** Virtual destructor. */
	virtual ~FUdpSocketReceiver()
	{
		if (Thread != nullptr)
		{
			Thread->Kill(true);
			delete Thread;
		}
	}

public:

	/** Set the maximum size allocated to read off of the socket. */
	void SetMaxReadBufferSize(uint32 InMaxReadBufferSize)
	{
		MaxReadBufferSize = InMaxReadBufferSize;
	}

	/** Set the desired Thread Stack Size. Only read when Start() is called. */
	void SetThreadStackSize(uint32 InTheadStackSize)
	{
		ThreadStackSize = InTheadStackSize;
	}

	/** Start the receiver thread. */
	void Start()
	{
		Thread = FRunnableThread::Create(this, *ThreadName, ThreadStackSize, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
	}

	/**
	 * Returns a delegate that is executed when data has been received.
	 *
	 * This delegate must be bound before the receiver thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 *
	 * @return The delegate.
	 */
	FOnSocketDataReceived& OnDataReceived()
	{
		check(Thread == nullptr);
		return DataReceivedDelegate;
	}

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}

	virtual bool Init() override
	{
		return true;
	}

	virtual uint32 Run() override
	{
		while (!bStopping.load(std::memory_order_acquire))
		{
			Update(WaitTime);
		}

		return 0;
	}

	virtual void Stop() override
	{
		bStopping.store(true, std::memory_order_release);
	}

	virtual void Exit() override { }

protected:

	/** Update this socket receiver. */
	void Update(const FTimespan& SocketWaitTime)
	{
		if (!Socket->Wait(ESocketWaitConditions::WaitForRead, SocketWaitTime))
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FUdpSocketReceiver_Update);
		TSharedRef<FInternetAddr> Sender = SocketSubsystem->CreateInternetAddr();
		uint32 Size;

		while (Socket->HasPendingData(Size))
		{
			FArrayReaderPtr Reader = MakeShared<FArrayReader, ESPMode::ThreadSafe>(true);
			Reader->SetNumUninitialized(FMath::Min(Size, MaxReadBufferSize));

			int32 Read = 0;

			if (Socket->RecvFrom(Reader->GetData(), Reader->Num(), Read, *Sender))
			{
				ensure((uint32)Read <= MaxReadBufferSize);
				Reader->RemoveAt(Read, Reader->Num() - Read, EAllowShrinking::No);
				DataReceivedDelegate.ExecuteIfBound(Reader, FIPv4Endpoint(Sender));
			}
		}
	}

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override
	{
		Update(FTimespan::Zero());
	}

private:

	/** The network socket. */
	FSocket* Socket;

	/** Pointer to the socket sub-system. */
	ISocketSubsystem* SocketSubsystem;

	/** Flag indicating that the thread is stopping. */
	std::atomic<bool> bStopping;

	/** The thread object. */
	FRunnableThread* Thread;

	/** The receiver thread's name. */
	FString ThreadName;

	/** The amount of time to wait for inbound packets. */
	FTimespan WaitTime;

	/** The maximum read buffer size used to read the socket. */
	uint32 MaxReadBufferSize = 65507u;

	/** Thread stack size to use. */
	uint32 ThreadStackSize = 128 * 1024;

private:

	/** Holds the data received delegate. */
	FOnSocketDataReceived DataReceivedDelegate;
};
