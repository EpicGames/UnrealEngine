// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "Sockets.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"

/**
 * Asynchronously sends data to an UDP socket.
 *
 * @todo networking: gmp: implement rate limits
 */
class FUdpSocketSender
	: public FRunnable
	, private FSingleThreadRunnable
{
	// Structure for outbound packets.
	struct FPacket
	{
		/** Holds the packet's data. */
		TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> Data;

		/** Holds the recipient. */
		FIPv4Endpoint Recipient;

		/** Default constructor. */
		FPacket() { }

		/** Creates and initializes a new instance. */
		FPacket(const TSharedRef<TArray<uint8>, ESPMode::ThreadSafe>& InData, const FIPv4Endpoint& InRecipient)
			: Data(InData)
			, Recipient(InRecipient)
		{ }
	};

public:

	/**
	 * Creates and initializes a new socket sender.
	 *
	 * @param InSocket The UDP socket to use for sending data.
	 * @param ThreadDescription The thread description text (for debugging).
	 */
	FUdpSocketSender(FSocket* InSocket, const TCHAR* ThreadDescription)
		: SendRate(0)
		, Socket(InSocket)
		, Stopping(false)
		, WaitTime(FTimespan::FromMilliseconds(100))
	{
		check(Socket != nullptr);
		check(Socket->GetSocketType() == SOCKTYPE_Datagram);

		int32 NewSize = 0;
		Socket->SetSendBufferSize(512 * 1024, NewSize);

		WorkEvent = FPlatformProcess::GetSynchEventFromPool();
		Thread = FRunnableThread::Create(this, ThreadDescription, 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
	}

	/** Virtual destructor. */
	virtual ~FUdpSocketSender()
	{
		if (Thread != nullptr)
		{
			Thread->Kill(true);
			delete Thread;
		}

		FPlatformProcess::ReturnSynchEventToPool(WorkEvent);
		WorkEvent = nullptr;
	}

public:

	/**
	 * Gets the maximum send rate (in bytes per second).
	 *
	 * @return Current send rate.
	 */
	uint32 GetSendRate() const
	{
		return SendRate;
	}

	/**
	 * Gets the current throughput (in bytes per second).
	 *
	 * @return Current throughput.
	 */
	uint32 GetThroughput() const
	{
		return Throughput;
	}

	/**
	 * Sends data to the specified recipient.
	 *
	 * @param Data The data to send.
	 * @param Recipient The recipient.
	 * @return true if the data will be sent, false otherwise.
	 */
	bool Send(const TSharedRef<TArray<uint8>, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& Recipient)
	{
		if (!Stopping && SendQueue.Enqueue(FPacket(Data, Recipient)))
		{
			WorkEvent->Trigger();

			return true;
		}

		return false;
	}

	/**
	 * Sets the send rate (in bytes per second).
	 *
	 * @param Rate The new send rate (0 = unlimited).
	 * @see SetWaitTime
	 */
	void SetSendRate(uint32 Rate)
	{
		SendRate = Rate;
	}

	/**
	 * Sets the maximum time span to wait for work items.
	 *
	 * @param Timespan The wait time.
	 * @see SetSendRate
	 */
	void SetWaitTime(const FTimespan& Timespan)
	{
		WaitTime = Timespan;
	}

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}

	virtual bool Init() override
	{
		return true;
	}

	virtual uint32 Run() override
	{
		while (!Stopping)
		{
			if (!Update(WaitTime))
			{
				Stopping = true;

				return 0;
			}

			WorkEvent->Wait(WaitTime);
		}

		return 0;
	}

	virtual void Stop() override
	{
		Stopping = true;
		WorkEvent->Trigger();
	}

	virtual void Exit() override { }

protected:

	/**
	 * Update this socket sender.
	 *
	 * @param Time to wait for the socket.
	 * @return true on success, false otherwise.
	 */
	bool Update(const FTimespan& SocketWaitTime)
	{
		while (!SendQueue.IsEmpty())
		{
			if (!Socket->Wait(ESocketWaitConditions::WaitForWrite, SocketWaitTime))
			{
				break;
			}

			FPacket Packet;
			int32 Sent = 0;

			SendQueue.Dequeue(Packet);
			Socket->SendTo(Packet.Data->GetData(), Packet.Data->Num(), Sent, *Packet.Recipient.ToInternetAddr());

			if (Sent != Packet.Data->Num())
			{
				return false;
			}
		}

		return true;
	}

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override
	{
		Update(FTimespan::Zero());
	}

private:

	/** The send queue. */
	TQueue<FPacket, EQueueMode::Mpsc> SendQueue;

	/** The send rate. */
	uint32 SendRate;

	/** The network socket. */
	FSocket* Socket;

	/** Flag indicating that the thread is stopping. */
	bool Stopping;

	/** The thread object. */
	FRunnableThread* Thread;

	/** The current throughput. */
	uint32 Throughput;

	/** The amount of time to wait for inbound packets. */
	FTimespan WaitTime;

	/** An event signaling that inbound messages need to be processed. */
	FEvent* WorkEvent;
};
