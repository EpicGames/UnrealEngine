// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IPv4/IPv4SubnetMask.h"
#include "Internationalization/Text.h"
#include "Serialization/Archive.h"

/**
 * Implements an IPv4 address.
 */
struct FIPv4Address
{
	union
	{
		/** The IP address value as A.B.C.D components. */
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
	#ifdef _MSC_VER
			uint8 D, C, B, A;
	#else
			uint8 D GCC_ALIGN(4);
			uint8 C, B, A;
	#endif
#else
			uint8 A, B, C, D;
#endif
		};

		/** The IP address value in host byte order. */
		uint32 Value;
	};

public:
	
	/** Default constructor. */
	FIPv4Address() { }

	/**
	 * Creates and initializes a new IPv4 address with the specified components.
	 *
	 * The created IP address has the value A.B.C.D.
	 *
	 * @param InA The first component.
	 * @param InB The second component.
	 * @param InC The third component.
	 * @param InD The fourth component.
	 */
	FIPv4Address(uint8 InA, uint8 InB, uint8 InC, uint8 InD)
#if PLATFORM_LITTLE_ENDIAN
		: D(InD)
		, C(InC)
		, B(InB)
		, A(InA)
#else
		: A(InA)
		, B(InB)
		, C(InC)
		, D(InD)
#endif // PLATFORM_LITTLE_ENDIAN
	{ }

	/**
	 * Creates and initializes a new IPv4 address with the specified value.
	 *
	 * @param InValue The address value (in host byte order).
	 */
	FIPv4Address(uint32 InValue)
		: Value(InValue)
	{ }

public:

	/**
	 * Compares this IPv4 address with the given address for equality.
	 *
	 * @param Other The address to compare with.
	 * @return true if the addresses are equal, false otherwise.
	 */
	bool operator==(const FIPv4Address& Other) const
	{
		return (Value == Other.Value);
	}

	/**
	 * Compares this IPv4 address with the given address for inequality.
	 *
	 * @param Other The address to compare with.
	 * @return true if the addresses are not equal, false otherwise.
	 */
	bool operator!=(const FIPv4Address& Other) const
	{
		return (Value != Other.Value);
	}

	/**
	 * OR operator for masking IP addresses with a subnet mask.
	 *
	 * @param SubnetMask The subnet mask.
	 * @return The masked IP address.
	 */
	FIPv4Address operator|(const FIPv4SubnetMask& SubnetMask) const
	{
		return FIPv4Address(Value | SubnetMask.Value);
	}

	/**
	 * AND operator for masking IP addresses with a subnet mask.
	 *
	 * @param SubnetMask The subnet mask.
	 * @return The masked IP address.
	 */
	FIPv4Address operator&(const FIPv4SubnetMask& SubnetMask) const
	{
		return FIPv4Address(Value & SubnetMask.Value);
	}

	/**
	 * Serializes the given IPv4 address from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param IpAddress The address to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIPv4Address& IpAddress)
	{
		return Ar << IpAddress.Value;
	}

public:

	/**
	 * Checks whether this IP address is a global multicast address.
	 *
	 * Global multicast addresses are in the range 224.0.1.0 to 238.255.255.255.
	 *
	 * @return true if it is a global multicast address, false otherwise.
	 * @see IsLinkLocalMulticast, IsOrganizationLocalMulticast, IsSiteLocalMulticast
	 */
	bool IsGlobalMulticast() const
	{
		return (((A >= 224) && A <= 238) && !((A == 224) && (B == 0) && (C == 0)));
	}

	/**
	 * Checks whether this IP address is link local.
	 *
	 * Link local addresses are in the range 169.254.0.0/16
	 *
	 * @return true if it is link local, false otherwise.
	 */
	bool IsLinkLocal() const
	{
		return ((A == 169) && (B == 254));
	}

	/**
	 * Checks whether this IP address is a link local multicast address.
	 *
	 * Link local multicast addresses have the form 224.0.0.x.
	 *
	 * @return true if it is a link local multicast address, false otherwise.
	 * @see IsGlobalMulticast, IsOrganizationLocalMulticast, IsSiteLocalMulticast
	 */
	bool IsLinkLocalMulticast() const
	{
		return ((A >= 224) && (B == 0) && (C == 0));
	}

	/**
	 * Checks whether this IP address is a loopback address.
	 *
	 * Loopback addresses have the form 127.x.x.x
	 *
	 * @return true if it is a loopback address, false otherwise.
	 * @see IsMulticastAddress
	 */
	bool IsLoopbackAddress() const
	{
		return (A == 127);
	}

	/**
	 * Checks whether this IP address is a multicast address.
	 *
	 * Multicast addresses are in the range 224.0.0.0 to 239.255.255.255.
	 *
	 * @return true if it is a multicast address, false otherwise.
	 * @see IsLoopbackAddress
	 */
	bool IsMulticastAddress() const
	{
		return ((A >= 224) && (A <= 239));
	}

	/**
	 * Checks whether this IP address is a organization local multicast address.
	 *
	 * Organization local multicast addresses are in the range 239.192.x.x to 239.195.x.x.
	 *
	 * @return true if it is a site local multicast address, false otherwise.
	 * @see IsGlobalMulticast, IsLinkLocalMulticast, IsSiteLocalMulticast
	 */
	bool IsOrganizationLocalMulticast() const
	{
		return ((A == 239) && (B >= 192) && (B <= 195));
	}

	/**
	 * Checks whether this IP address is a site local address.
	 *
	 * Site local addresses have one of the following forms:
	 *		10.x.x.x
	 *		172.16-31.x.x
	 *		192.168.x.x
	 *
	 * @return true if it is a site local address, false otherwise.
	 */
	bool IsSiteLocalAddress() const
	{
		return ((A == 10) || ((A == 172) && ((B >= 16) && (B <= 31))) || ((A == 192) && (B == 168)));
	}

	/**
	 * Checks whether this IP address is a site local multicast address.
	 *
	 * Site local multicast addresses have the form 239.255.0.x.
	 *
	 * @return true if it is a site local multicast address, false otherwise.
	 * @see IsGlobalMulticast, IsLinkLocalMulticast, IsOrganizationLocalMulticast
	 */
	bool IsSiteLocalMulticast() const
	{
		return ((A == 239) && (B == 255));
	}

	/**
	 * Checks whether this IP address is the multicast address we use for the Session Frontend
	 * Which is 230.0.0.1
	 *
	 * @return true if this address matches, false otherwise.
	 */
	bool IsSessionFrontendMulticast() const
	{
		return (A == 230 && B == 0 && C == 0 && D == 1);
	}

	/**
	 * Gets the string representation for this address.
	 *
	 * @return String representation.
	 * @see Parse, ToText
	 */
	NETWORKING_API FString ToString() const;

	/**
	* Gets the display text representation.
	*
	* @return Text representation.
	* @see ToString
	*/
	FText ToText() const
	{
		return FText::FromString(ToString());
	}

public:

	/**
	 * Gets the hash for specified IPv4 address.
	 *
	 * @param Address The address to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const FIPv4Address& Address)
	{
		return Address.Value;
	}

public:

	/**
	 * Converts a string to an IPv4 address.
	 *
	 * @param AddressString The string to convert.
	 * @param OutAddress Will contain the parsed address.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ToString
	 */
	static NETWORKING_API bool Parse(const FString& AddressString, FIPv4Address& OutAddress);

public:

	/** Defines the wild card address, which is 0.0.0.0	*/
	static NETWORKING_API const FIPv4Address Any;

	/** Defines the internal loopback address, which is 127.0.0.1 */
	static NETWORKING_API const FIPv4Address InternalLoopback;

	/** Defines the broadcast address for the 'zero network' (i.e. LAN), which is 255.255.255.255. */
	static NETWORKING_API const FIPv4Address LanBroadcast;
};
