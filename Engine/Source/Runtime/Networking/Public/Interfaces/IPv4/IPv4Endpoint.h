// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IPAddress.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "SocketSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

/**
 * Implements an endpoint for IPv4 networks.
 *
 * An endpoint consists of an IPv4 address and a port number.
 *
 * @todo gmp: add IPv6 support and rename this to FIpEndpoint?
 */
struct FIPv4Endpoint
{
	/** Holds the endpoint's IP address. */
	FIPv4Address Address;

	/** Holds the endpoint's port number. */
	uint16 Port; 

public:

	/** Default constructor. */
	FIPv4Endpoint() { }

	/**
	 * Creates and initializes a new IPv4 endpoint with the specified NetID and port.
	 *
	 * @param InAddress The endpoint's IP address.
	 * @param InPort The endpoint's port number.
	 */
	FIPv4Endpoint(const FIPv4Address& InAddress, uint16 InPort)
		: Address(InAddress)
		, Port(InPort)
	{ }

	/**
	 * Creates and initializes a new IPv4 endpoint from a given FInternetAddr object.
	 *
	 * Note: this constructor will be removed after the socket subsystem has been refactored.
	 *
	 * @param InternetAddr The Internet address.
	 */
	FIPv4Endpoint(const TSharedPtr<FInternetAddr>& InternetAddr)
	{
		check(InternetAddr.IsValid());

		int32 OutPort;
		InternetAddr->GetIp(Address.Value);
		InternetAddr->GetPort(OutPort);
		Port = static_cast<uint16>(OutPort);
	}

public:

	/**
	 * Compares this IPv4 endpoint with the given endpoint for equality.
	 *
	 * @param Other The endpoint to compare with.
	 * @return true if the endpoints are equal, false otherwise.
	 */
	bool operator==(const FIPv4Endpoint& Other) const
	{
		return ((Address == Other.Address) && (Port == Other.Port));
	}

	/**
	 * Compares this IPv4 address with the given endpoint for inequality.
	 *
	 * @param Other The endpoint to compare with.
	 * @return true if the endpoints are not equal, false otherwise.
	 */
	bool operator!=(const FIPv4Endpoint& Other) const
	{
		return ((Address != Other.Address) || (Port != Other.Port));
	}

	/**
	 * Serializes the given endpoint from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Endpoint The endpoint to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIPv4Endpoint& Endpoint)
	{
		return Ar << Endpoint.Address << Endpoint.Port;
	}

public:

	/**
	 * Converts this endpoint to an FInternetAddr object.
	 *
	 * Note: this method will be removed after the socket subsystem is refactored.
	 *
	 * @return Internet address object representing this endpoint.
	 */
	TSharedRef<FInternetAddr> ToInternetAddr() const
	{
		check(CachedSocketSubsystem != nullptr && "Networking module not loaded and initialized");

		TSharedRef<FInternetAddr> InternetAddr = CachedSocketSubsystem->CreateInternetAddr();
		{
			InternetAddr->SetIp(Address.Value);
			InternetAddr->SetPort(Port);
		}

		return InternetAddr;
	}

	/**
	 * Converts this endpoint to an FInternetAddr object.
	 *
	 * Note: this method will be removed after the socket subsystem is refactored.
	 *
	 * @return Internet address object representing this endpoint.
	 */
	TSharedRef<FInternetAddr> ToInternetAddrIPV4() const
	{
		check(CachedSocketSubsystem != nullptr && "Networking module not loaded and initialized");

		TSharedRef<FInternetAddr> InternetAddr = CachedSocketSubsystem->CreateInternetAddr();
		{
			TArray<uint8> RawAddress;
			RawAddress.Add(Address.A);
			RawAddress.Add(Address.B);
			RawAddress.Add(Address.C);
			RawAddress.Add(Address.D);
			InternetAddr->SetRawIp(RawAddress);
			InternetAddr->SetPort(Port);
		}

		return InternetAddr;
	}

	/**
	 * Gets a string representation for this endpoint.
	 *
	 * @return String representation.
	 * @see Parse, ToText
	 */
	NETWORKING_API FString ToString() const;

	/**
	 * Gets the display text representation.
	 *
	 * @return Text representation.
	 * @see ToString
	 */
	FText ToText() const
	{
		return FText::FromString(ToString());
	}

public:

	/**
	 * Gets the hash for specified IPv4 endpoint.
	 *
	 * @param Endpoint The endpoint to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const FIPv4Endpoint& Endpoint)
	{
		return HashCombine(GetTypeHash(Endpoint.Address), GetTypeHash(Endpoint.Port));
	}

public:

	/** Initializes the IP endpoint functionality. */
	static NETWORKING_API void Initialize();

	/**
	 * Converts a string to an IPv4 endpoint.
	 *
	 * @param EndpointString The string to convert.
	 * @param OutEndpoint Will contain the parsed endpoint.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ToString
	 */
	static NETWORKING_API bool Parse(const FString& EndpointString, FIPv4Endpoint& OutEndpoint);

	/**
	 * Resolves a hostname and port to an IPv4 endpoint.
	 *
	 * @param HostAndPortString The host and port to resolve.
	 * @param OutEndpoint Will contain the resolved endpoint.
	 * @return true if the host and port were parsed and resolved successfully, false otherwise.
	 */
	static NETWORKING_API bool FromHostAndPort(const FString& HostAndPortString, FIPv4Endpoint& OutEndpoint);

public:

	/** Defines the wild card endpoint, which is 0.0.0.0:0 */
	static NETWORKING_API const FIPv4Endpoint Any;

private:

	/** ISocketSubsystem::Get() is not thread-safe, so we cache it here. */
	static NETWORKING_API ISocketSubsystem* CachedSocketSubsystem;
};
