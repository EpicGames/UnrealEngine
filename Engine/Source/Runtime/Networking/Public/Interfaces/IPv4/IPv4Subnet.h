// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4SubnetMask.h"
#include "Internationalization/Text.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"

/**
 * Implements a IPv4 subnet descriptor.
 *
 * @todo gmp: add IsValid() method
 */
struct FIPv4Subnet
{
	/** Holds the subnet's address. */
	FIPv4Address Address;

	/** Holds the subnet's mask. */
	FIPv4SubnetMask Mask;

public:

	/** Default constructor. */
	FIPv4Subnet() { }

	/**
	 * Creates and initializes a new IPv4 subnet with the specified address and mask.
	 *
	 * @param InAddress The subnet's address.
	 * @param InMask The subnet's mask.
	 */
	FIPv4Subnet(const FIPv4Address& InAddress, const FIPv4SubnetMask& InMask)
		: Address(InAddress)
		, Mask(InMask)
	{ }

public:

	/**
	 * Compares this IPv4 subnet descriptor with the given subnet for equality.
	 *
	 * @param Other The subnet to compare with.
	 * @return true if the subnets are equal, false otherwise.
	 */
	bool operator==(const FIPv4Subnet& Other) const
	{
		return ((Address == Other.Address) && (Mask == Other.Mask));
	}

	/**
	 * Compares this IPv4 subnet descriptor with the given subnet for inequality.
	 *
	 * @param Other The subnet to compare with.
	 * @return true if the subnets are not equal, false otherwise.
	 */
	bool operator!=(const FIPv4Subnet& Other) const
	{
		return ((Address != Other.Address) || (Mask != Other.Mask));
	}

	/**
	 * Serializes the given subnet descriptor from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Subnet The subnet descriptor to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIPv4Subnet& Subnet)
	{
		return Ar << Subnet.Address << Subnet.Mask;
	}

public:

	/**
	* Get the broadcast address for this subnet.
	*
	* @return The broadcast address.
	*/
	FIPv4Address BroadcastAddress() const
	{
		return (Address | ~Mask);
	}

	/**
	 * Checks whether the subnet contains the specified IP address.
	 *
	 * @param TestAddress The address to check.
	 * @return true if the subnet contains the address, false otherwise.
	 */
	bool ContainsAddress(const FIPv4Address& TestAddress)
	{
		return ((Address & Mask) == (TestAddress & Mask));
	}

	/**
	 * Converts this IP subnet to its string representation.
	 *
	 * @return String representation.
	 * @see Parse, ToText
	 */
	NETWORKING_API FString ToString() const;

	/**
	* Gets the display text representation.
	*
	* @return Text representation.
	* @see ToString
	*/
	FText ToText() const
	{
		return FText::FromString(ToString());
	}

public:

	/**
	 * Gets the hash for specified IPv4 subnet.
	 *
	 * @param Subnet The subnet to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const FIPv4Subnet& Subnet)
	{
		return HashCombine(GetTypeHash(Subnet.Address), GetTypeHash(Subnet.Mask));
	}

public:

	/**
	 * Converts a string to an IPv4 subnet.
	 *
	 * @param SubnetString The string to convert.
	 * @param OutSubnet Will contain the parsed subnet.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ToString
	 */
	static NETWORKING_API bool Parse(const FString& SubnetString, FIPv4Subnet& OutSubnet);
};
