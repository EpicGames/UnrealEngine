// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"

namespace EIPv4SubnetClasses
{
	/**
	 * Enumerates IPv4 subnet classes.
	 */
	enum Type
	{
		/** Invalid subnet mask. */
		Invalid,

		/** Class A subnet. */
		ClassA,

		/** Class B subnet. */
		ClassB,

		/** Class C subnet. */
		ClassC
	};
}


/**
 * Implements an IPv4 subnet mask.
 */
struct FIPv4SubnetMask
{
	union
	{
		/** The subnet mask value as A.B.C.D components. */
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
	#ifdef _MSC_VER
			uint8 D, C, B, A;
	#else
			uint8 D GCC_ALIGN(4);
			uint8 C, B, A;
	#endif
#else
			uint8 A, B, C, D;
#endif
		};

		/** The subnet mask value in host byte order. */
		int32 Value;
	};

public:

	/** Default constructor. */
	FIPv4SubnetMask() { }

	/**
	 * Creates and initializes a new IPv4 subnet mask with the specified components.
	 *
	 * The created subnet mask has the value A.B.C.D.
	 *
	 * @param InA The first component.
	 * @param InB The second component.
	 * @param InC The third component.
	 * @param InD The fourth component.
	 */
	FIPv4SubnetMask(uint8 InA, uint8 InB, uint8 InC, uint8 InD)
#if PLATFORM_LITTLE_ENDIAN
		: D(InD)
		, C(InC)
		, B(InB)
		, A(InA)
#else
		: A(InA)
		, B(InB)
		, C(InC)
		, D(InD)
#endif // PLATFORM_LITTLE_ENDIAN
	{ }
	
	/**
	 * Creates and initializes a new IPv4 subnet mask with the specified value.
	 *
	 * @param InValue The address value (in host byte order).
	 */
	FIPv4SubnetMask(uint32 InValue)
		: Value(InValue)
	{ }

public:

	/**
	 * Compares this subnet mask with the given mask for equality.
	 *
	 * @param Other The subnet mask to compare with.
	 * @return true if the subnet masks are equal, false otherwise.
	 */
	bool operator==(const FIPv4SubnetMask& Other) const
	{
		return (Value == Other.Value);
	}

	/**
	 * Compares this subnet mask with the given address for inequality.
	 *
	 * @param Other The subnet mask to compare with.
	 * @return true if the subnet masks are not equal, false otherwise.
	 */
	bool operator!=(const FIPv4SubnetMask& Other) const
	{
		return (Value != Other.Value);
	}

	/**
	 * Returns an inverted subnet mask.
	 *
	 * @return Inverted subnet mask.
	 */
	FIPv4SubnetMask operator~() const
	{
		return FIPv4SubnetMask(~Value);
	}

	/**
	 * Serializes the given subnet mask from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param SubnetMask The subnet mask to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FIPv4SubnetMask& SubnetMask)
	{
		return Ar << SubnetMask.Value;
	}

public:

	/**
	 * Gets the subnet class that this mask specifies.
	 *
	 * @return Subnet class.
	 */
	EIPv4SubnetClasses::Type GetClass() const
	{
		if (A == 255)
		{
			if (B == 255)
			{
				if (C == 255)
				{
					return EIPv4SubnetClasses::ClassC;
				}

				return EIPv4SubnetClasses::ClassB;
			}

			return EIPv4SubnetClasses::ClassA;
		}

		return EIPv4SubnetClasses::Invalid;
	}

	/**
	 * Converts this IP address to its string representation.
	 *
	 * @return String representation.
	 * @see Parse, ToText
	 */
	NETWORKING_API FString ToString() const;

	/**
	* Gets the display text representation.
	*
	* @return Text representation.
	* @see ToString
	*/
	FText ToText() const
	{
		return FText::FromString(ToString());
	}

public:

	/**
	 * Gets the hash for specified IPv4 subnet mask.
	 *
	 * @param SubnetMask The subnet mask to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const FIPv4SubnetMask& SubnetMask)
	{
		return GetTypeHash(SubnetMask.Value);
	}

public:
	
	/**
	 * Converts a string to an IPv4 subnet mask.
	 *
	 * @param MaskString The string to convert.
	 * @param OutMask Will contain the parsed subnet mask.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ToString
	 */
	static NETWORKING_API bool Parse(const FString& MaskString, FIPv4SubnetMask& OutMask);
};
