// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"

/**
 * Implements a Steam endpoint.
 */
class FSteamEndpoint
{
public:

	/** Default constructor. */
	FSteamEndpoint() { }

	/**
	 * Creates and initializes a new Steam address with the specified components.
	 *
	 * @param InUniqueNetId The unique network identifier.
	 * @param InSteamChannel The Steam channel number.
	 */
	FSteamEndpoint(uint64 InUniqueNetId, int32 InSteamChannel)
		: SteamChannel(InSteamChannel)
		, UniqueNetId(InUniqueNetId)
	{ }

public:

	/**
	 * Compares this Steam address with the given address for equality.
	 *
	 * @param Other The address to compare with.
	 * @return true if the addresses are equal, false otherwise.
	 */
	bool operator==(const FSteamEndpoint& Other) const
	{
		return ((SteamChannel == Other.SteamChannel) && (UniqueNetId == Other.UniqueNetId));
	}

	/**
	 * Compares this Steam address with the given address for inequality.
	 *
	 * @param Other The address to compare with.
	 * @return true if the addresses are not equal, false otherwise.
	 */
	bool operator!=(const FSteamEndpoint& Other) const
	{
		return ((SteamChannel != Other.SteamChannel) || (UniqueNetId != Other.UniqueNetId));
	}

	/**
	 * Serializes the given Steam address from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param IpAddress The address to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<(FArchive& Ar, FSteamEndpoint& SteamEndpoint)
	{
		return Ar << SteamEndpoint.UniqueNetId << SteamEndpoint.SteamChannel;
	}

public:

	/**
	 * Gets the Steam channel number.
	 *
	 * @return Steam channel number.
	 * @see GetUniqueNetId
	 */
	int32 GetSteamChannel() const
	{
		return SteamChannel;
	}

	/**
	 * Gets the unique network identifier.
	 *
	 * @return Unique network identifier.
	 * @see GetSteamChannel
	 */
	uint64 GetUniqueNetId() const
	{
		return UniqueNetId;
	}


	/**
	 * Gets the string representation for this address.
	 *
	 * @return String representation.
	 */
	NETWORKING_API FString ToString() const;

public:

	/**
	 * Gets the hash for specified Steam address.
	 *
	 * @param Address - The address to get the hash for.
	 *
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash(const FSteamEndpoint& Address)
	{
		return GetTypeHash(Address.UniqueNetId) + GetTypeHash(Address.SteamChannel) * 23;
	}

private:

	/** Holds the Steam channel number. */
	int32 SteamChannel;

	/** Holds the unique network identifier. */
	uint64 UniqueNetId;
};
