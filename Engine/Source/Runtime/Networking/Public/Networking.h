// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public Dependencies
 *****************************************************************************/

#include "Core.h"
#include "Modules/ModuleManager.h"
#include "Sockets.h"
#include "SocketSubsystem.h"


/* Public Includes
 *****************************************************************************/

#include "Interfaces/IPv4/IPv4SubnetMask.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Interfaces/IPv4/IPv4Subnet.h"
#include "Interfaces/Steam/SteamEndpoint.h"
#include "Interfaces/INetworkingModule.h"


/* Common
 *****************************************************************************/

#include "Common/TcpSocketBuilder.h"
#include "Common/TcpListener.h"
#include "Common/UdpSocketReceiver.h"
#include "Common/UdpSocketSender.h"
#include "Common/UdpSocketBuilder.h"
