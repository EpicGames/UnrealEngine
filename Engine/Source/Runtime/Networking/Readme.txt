This module is an internal R&D effort to explore how the socket subsystem could be improved.
Production use is NOT encouraged at this point.